[OnlineSubsystem]
DefaultPlatformService=GDK

