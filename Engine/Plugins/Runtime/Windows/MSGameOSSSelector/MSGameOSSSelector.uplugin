{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "Online Subsystem Selector - GDK (Xbox PC)",
	"Description": "Automatically switches to OnlineSubsystemGDK when the game is installed via the Xbox PC App.",
	"CreatedBy": "Epic Games, Inc.",
	"Category": "Platform",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"RequiresBuildPlatform": false,
	"CanContainContent": false,
	"IsBetaVersion": true,
	"Installed": false,
	"SupportedTargetPlatforms": [ "Win64" ],
	"Modules": [
		{
			"Name": "MSGameOSSSelector",
			"Type": "RuntimeNoCommandlet",
			"LoadingPhase": "PostConfigInit",
			"PlatformAllowList": [ "Win64" ]
		}
	],
	"Plugins": [
		{
			"Name": "OnlineSubsystemGDK",
			"Enabled": true,
			"Optional": true
		}
	]
}