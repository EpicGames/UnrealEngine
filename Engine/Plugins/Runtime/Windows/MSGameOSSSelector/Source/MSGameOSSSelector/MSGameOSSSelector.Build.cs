// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class MSGameOSSSelector : ModuleRules
{
	public MSGameOSSSelector(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(new[]{
			"Projects",
			"Core",
			"GRDK",
		});

		//note: this will also require adding to your project's WindowsGame.ini or there'll be a warning during staging
		//[Staging]
		//+AllowedConfigFiles=MyGame/Config/Windows/MSGameOSS/WindowsEngine.ini
		RuntimeDependencies.Add("$(ProjectDir)/Config/Windows/MSGameOSS/*.ini", StagedFileType.UFS);
	}
}
