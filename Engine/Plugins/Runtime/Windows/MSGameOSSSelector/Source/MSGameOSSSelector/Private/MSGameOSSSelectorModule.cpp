// Copyright Epic Games, Inc. All Rights Reserved.

#include "MSGameOSSSelectorModule.h"
#include "Interfaces/IPluginManager.h"
#include "HAL/FileManager.h"
#include "HAL/Platform.h"
#include "Misc/Paths.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CommandLine.h"
#include "Logging/LogCategory.h"

THIRD_PARTY_INCLUDES_START
#include "Microsoft/AllowMicrosoftPlatformTypes.h"
#include <appmodel.h>
#include "Microsoft/HideMicrosoftPlatformTypes.h"
THIRD_PARTY_INCLUDES_END


/*
*
*	Run with -ForceOSSGDK to apply the configuration regardless of whether we are a packaged process
*	or -ForceNoOSSGDK to leave the configuration as default even when packaged
*
*
*
*   You will also need this line added to your game's WindowsEngine.ini to ensure the OSS configuration
*	is patched before it's used by the OSS module
*
*	[OnlineSubsystem]
*	+AdditionalModulesToLoad=MSGameOSSSelector
*
*
*
*	Additional ini files can be placed in this folder, with a Windows prefix (e.g. WindowsEngine.ini):
*	YourProject/Config/Windows/MSGameOSS/
*
*	...these should have an appropriate [Staging] entry in your project's main WindowsGame.ini:
*	+AllowedConfigFiles=YourProject/Config/Windows/MSGameOSS/WindowsEngine.ini
*
*	These additional ini files can be used to configure other online related items such as:
*	DefaultPlatformService, LocalPlatformName, NetDriverDefinitions etc,
*
*	It can also override other key items depending on the needs of the project, such as:
*
*	[PlatformFeatures]
*	SaveGameSystemModule=GDKSaveGameSystem
*
*	[StreamingInstall]
*	DefaultProviderName=GDKPackageChunkInstall
* 
*   And finally, you can also load additional modules:
* 
*	[MSGameOSSSelector]
*	+AdditionalModulesToLoad=MyCustomGDKModule
*/



DEFINE_LOG_CATEGORY_STATIC(MSGameOSSSelector, Log, All);


class FMSGameOSSSelectorModule : public IMSGameOSSSelectorModule
{
public:

	virtual void StartupModule() override
	{
		const bool bForceNoOSSGDK = FParse::Param(FCommandLine::Get(), TEXT("ForceNoOSSGDK"));
		if (!bForceNoOSSGDK)
		{
			const bool bForceOSSGDK = FParse::Param(FCommandLine::Get(), TEXT("ForceOSSGDK"));
			if (bForceOSSGDK || IsPackagedProcess())
			{
				bHasModifiedConfiguration = TryModifyConfiguration();

				// if the configuration was modified, load in any additional modules the user needs
				if (bHasModifiedConfiguration)
				{
					TArray<FString> AdditionalModulesToLoad;
					GConfig->GetArray(TEXT("MSGameOSSSelector"), TEXT("AdditionalModulesToLoad"), AdditionalModulesToLoad, GEngineIni);
					for (const FString& AdditionalModule : AdditionalModulesToLoad)
					{
						if (FModuleManager::Get().ModuleExists(*AdditionalModule))
						{
							FModuleManager::Get().LoadModule(*AdditionalModule);
						}
					}
				}
			}
		}
	}

	virtual bool HasModifiedConfiguration() const override
	{
		return bHasModifiedConfiguration;
	}

private:

	bool TryModifyConfiguration() const
	{
		// make sure we are listed in the OSS module's prerequites, otherwise it will probably be too late for us to patch
		TArray<FString> AdditionalModulesToLoad;
		GConfig->GetArray(TEXT("OnlineSubsystem"), TEXT("AdditionalModulesToLoad"), AdditionalModulesToLoad, GEngineIni);
		if (!AdditionalModulesToLoad.Contains(TEXT("MSGameOSSSelector")))
		{
			UE_LOGF(MSGameOSSSelector, Error, "OSS configuration was not modified - make sure your WindowsEngine.ini has:\n[OnlineSubsystem]\n+AdditionalModulesToLoad=MSGameOSSSelector");
			return false;
		}

		// no point in switching if the game was built without GDK support
#if !WITH_GRDK
		UE_LOGF(MSGameOSSSelector, Error, "OSS configuration was not modified - game was not built with GDK support");
		return false;
#else

		// load ini files from the plugin's config directory
		const TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(TEXT("MSGameOSSSelector"));
		const FString PluginConfigDir = FPaths::GetPath(Plugin->GetDescriptorFileName()) / TEXT("Config");
		const int32 NumPluginConfigs = LoadConfigFiles(PluginConfigDir, TEXT("MSGameOSS"));

		// load ini files from the project's MSGameConfig directory
		const FString ProjectConfigDir = FPaths::Combine(FPaths::ProjectConfigDir(), TEXT("Windows"), TEXT("MSGameOSS"));
		const int32 NumProjectConfigs = LoadConfigFiles(ProjectConfigDir, TEXT("Windows"));

		// check results
		if ((NumPluginConfigs+NumProjectConfigs) == 0)
		{
			UE_LOGF(MSGameOSSSelector, Error, "OSS configuration was not modified - no config files added");
			return false;
		}
		else
		{
			UE_LOGF(MSGameOSSSelector, Log, "OSS configuration modified (%d config files)", NumPluginConfigs+NumProjectConfigs);
			return true;
		}
#endif
	}


	int32 LoadConfigFiles( const FString& ConfigDir, const TCHAR* ConfigFilePrefix ) const
	{
		TArray<FString> ConfigFiles;
		IFileManager::Get().FindFiles(ConfigFiles, *ConfigDir, *FString::Printf(TEXT("%s*.ini"), ConfigFilePrefix));

		int32 NumConfigs = 0;
		for (const FString& ConfigFile : ConfigFiles)
		{
			// get the base config file branch - Engine, Game etc. by stripping the path & prefix
			FString BaseConfigFile = FPaths::GetBaseFilename(ConfigFile);
			BaseConfigFile.RemoveFromStart(ConfigFilePrefix, ESearchCase::IgnoreCase);

			// find the config branch and add this ini file
			if (FConfigBranch* FoundConfig = GConfig->FindBranch(*BaseConfigFile, FString()))
			{
				if (FoundConfig->AddDynamicLayerToHierarchy( FPaths::Combine(ConfigDir, ConfigFile) ))
				{
					NumConfigs++;
				}
			}
		}

		UE_CLOG(NumConfigs < ConfigFiles.Num(), MSGameOSSSelector, Warning, TEXT("Only loaded %d/%d ini files from %s"), NumConfigs, ConfigFiles.Num(), *ConfigDir);
		return NumConfigs;
	}


	// Determines if we are an installed msixvc package without using the GDK runtime
	// (copy/paste of the function in the MSGameStore module, but don't want to pull that 
	// module in automically because it modifies the staging process)
	bool IsPackagedProcess() const
	{
		// editor will never be packaged 
#if WITH_EDITOR
		return false;
#else

		// get the kernel
		HMODULE hModule = ::GetModuleHandleW(TEXT("kernel32.dll"));
		if (hModule == nullptr)
		{
			return false;
		}

		// look up the GetCurrentPackageFullName function
PRAGMA_DISABLE_CAST_FUNCTION_TYPE_MISMATCH_WARNINGS	// unsafe conversion from 'type of expression' to 'type required'
		typedef LONG(WINAPI *GetCurrentPackageFullNameProc)(UINT32*, PWSTR);
		GetCurrentPackageFullNameProc fnGetCurrentPackageFullName = (GetCurrentPackageFullNameProc)::GetProcAddress(hModule, "GetCurrentPackageFullName");
PRAGMA_ENABLE_CAST_FUNCTION_TYPE_MISMATCH_WARNINGS

		if (fnGetCurrentPackageFullName == nullptr)
		{
			return false;
		}

		// request buffer size
		UINT32 BufferLength = 0;
		LONG Result = fnGetCurrentPackageFullName(&BufferLength, nullptr);
		if (Result != ERROR_INSUFFICIENT_BUFFER)
		{
			return false;
		}

		// prepare a buffer and get the package full name
		TUniquePtr<WCHAR[]> Buffer = MakeUnique<WCHAR[]>(BufferLength);
		Result = fnGetCurrentPackageFullName(&BufferLength, Buffer.Get());
		if (Result != ERROR_SUCCESS)
		{
			return false;
		}

		return true;
#endif //WITH_EDITOR
	}


	bool bHasModifiedConfiguration = false;
};

IMPLEMENT_MODULE(FMSGameOSSSelectorModule, MSGameOSSSelector);
