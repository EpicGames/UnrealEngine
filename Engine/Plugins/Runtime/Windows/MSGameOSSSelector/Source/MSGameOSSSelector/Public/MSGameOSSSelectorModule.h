// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

class MSGAMEOSSSELECTOR_API IMSGameOSSSelectorModule : public IModuleInterface
{
public:
	/**
	 * Singleton-like access to the OSS selector module instance
	 * @return Returns IMSGameOSSSelectorModule singleton instance, loading the module on demand if needed
	 */
	static inline IMSGameOSSSelectorModule& Get()
	{
		return FModuleManager::LoadModuleChecked<IMSGameOSSSelectorModule>("MSGameOSSSelector");
	}

	/**
	 * Returns whether the configuration patch was applied
	 */
	virtual bool HasModifiedConfiguration() const = 0;
};

